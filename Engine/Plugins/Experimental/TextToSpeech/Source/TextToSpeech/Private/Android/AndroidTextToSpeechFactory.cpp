// Copyright Epic Games, Inc. All Rights Reserved.

#if PLATFORM_ANDROID
#include "Android/AndroidTextToSpeechFactory.h"
#include "GenericPlatform/ITextToSpeechFactory.h"
#include "GenericPlatform/TextToSpeechBase.h"
#include "Flite/FliteTextToSpeech.h"

TSharedRef<FTextToSpeechBase> FAndroidTextToSpeechFactory::Create()
{
	return MakeShared<FFliteTextToSpeech>();
}
#endif
