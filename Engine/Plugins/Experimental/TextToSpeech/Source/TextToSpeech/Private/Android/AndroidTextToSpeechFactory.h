// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#if PLATFORM_ANDROID
#include "CoreMinimal.h"
#include "GenericPlatform/ITextToSpeechFactory.h"

class FTextToSpeechBase;
/** The platform default text to speech factory for Android platforms. */
class FAndroidTextToSpeechFactory : public ITextToSpeechFactory
{
public:
	FAndroidTextToSpeechFactory() = default;
	virtual ~FAndroidTextToSpeechFactory() = default;

	virtual TSharedRef<FTextToSpeechBase> Create() override;
};

typedef FAndroidTextToSpeechFactory FPlatformTextToSpeechFactory;
#endif
