// Copyright Epic Games, Inc. All Rights Reserved.
#include "Apple/AppleTextToSpeech.h"
#include "GenericPlatform/TextToSpeechBase.h"
#include "TextToSpeechLog.h"
#include "Apple/AppleStringUtils.h"
#include "Apple/ScopeAutoreleasePool.h"

#if PLATFORM_MAC
#include "Mac/CocoaThread.h"
#import <AppKit/AppKit.h>
#endif

#if PLATFORM_IOS
#include "Async/TaskGraphInterfaces.h"
#include "IOS/IOSAppDelegate.h"
#import <UIKit/UIKit.h>
#endif

#include "Apple/AppleMemoryObject.h"

// #import <Foundation/Foundation.h>
#import <AVFoundation/AVFoundation.h>

/** Defined in FTextToSpeechBase.cpp */
extern TMap<TextToSpeechId, TWeakPtr<FTextToSpeechBase>> ActiveTextToSpeechMap;

@interface FSpeechSynthesizerDelegate : FApplePlatformObject<AVSpeechSynthesizerDelegate>
-(id)initWithOwningTextToSpeechId:(TextToSpeechId)InOwningTextToSpeechId;
-(void)dealloc;
-(void)speechSynthesizer:(AVSpeechSynthesizer*)sender didFinishSpeechUtterance:(AVSpeechUtterance*)utterance;
@end

@implementation FSpeechSynthesizerDelegate
{
	TextToSpeechId _OwningTextToSpeechId;
}
APPLE_PLATFORM_OBJECT_ALLOC_OVERRIDES(FSpeechSynthesizerDelegate)
-(id)initWithOwningTextToSpeechId:(TextToSpeechId)InOwningTextToSpeechId
{
	if (self = [super init])
	{
		_OwningTextToSpeechId = InOwningTextToSpeechId;
	}
	return self;
}

-(void)dealloc
{
	_OwningTextToSpeechId = FTextToSpeechBase::InvalidTextToSpeechId;
	[super dealloc];
}

- (void)speechSynthesizer:(AVSpeechSynthesizer*)sender didFinishSpeechUtterance:(AVSpeechUtterance*)utterance
{
	// The announcement was completed successfully, not interrupted or manually stopped
	if (utterance && _OwningTextToSpeechId != FTextToSpeechBase::InvalidTextToSpeechId)
	{
		TextToSpeechId IdCopy = _OwningTextToSpeechId;
		const TMap<TextToSpeechId, TWeakPtr<FTextToSpeechBase>>& ActiveTextToSpeechMapRef = ActiveTextToSpeechMap;
#if PLATFORM_MAC
		GameThreadCall(^{
			const TWeakPtr<FTextToSpeechBase>* TTSPtr = ActiveTextToSpeechMapRef.Find(IdCopy);
			if (TTSPtr && TTSPtr->IsValid())
			{
				TTSPtr->Pin()->OnTextToSpeechFinishSpeaking_GameThread();
			}
		}, false);
#else // PLATFORM_IOS
		FFunctionGraphTask::CreateAndDispatchWhenReady([IdCopy, ActiveTextToSpeechMapRef]()
		{
			const TWeakPtr<FTextToSpeechBase>* TTSPtr = ActiveTextToSpeechMapRef.Find(IdCopy);
			if (TTSPtr && TTSPtr->IsValid())
			{
				TTSPtr->Pin()->OnTextToSpeechFinishSpeaking_GameThread();
			}
		}, TStatId(), NULL, ENamedThreads::GameThread);
#endif
	}
}

@end

FAppleTextToSpeech::FAppleTextToSpeech()
	: bIsSpeaking(false)
	, Volume(0.0f)
	, Rate(0.0f)
	, SpeechSynthesizer(nullptr)
	, SpeechSynthesizerDelegate(nullptr)
{

}

FAppleTextToSpeech::~FAppleTextToSpeech()
{
	// base class already takes care of checking if the TTS is active and calls deactivate
}

void FAppleTextToSpeech::Speak(const FString& InStringToSpeak)
{
	if (IsActive())
	{
		UE_LOGF(LogTextToSpeech, Verbose, "Apple TTS speak requested.");
		if (!InStringToSpeak.IsEmpty())
		{
			UE_LOGF(LogTextToSpeech, VeryVerbose, "String to speak: %ls", *InStringToSpeak);
			if (IsSpeaking())
			{
				StopSpeaking();
			}
			SCOPED_AUTORELEASE_POOL;
			NSString* Announcement = InStringToSpeak.GetNSString();
			ensureMsgf(Announcement, TEXT("Failed to convert FString to NSString for TTS."));
			AVSpeechUtterance* Utterance = [AVSpeechUtterance speechUtteranceWithString:Announcement];
			checkf(Utterance, TEXT("Failed to create an utterance for text to speech."));
			// for now we just use the default system language that's being used
			Utterance.voice = [AVSpeechSynthesisVoice voiceWithLanguage: [AVSpeechSynthesisVoice currentLanguageCode]];
			// If muted, set to volume to 0
			Utterance.volume = IsMuted() ? 0.0f : Volume;
			Utterance.rate = Rate;
			[SpeechSynthesizer speakUtterance:Utterance];
			bIsSpeaking = true;
		}
	}
}

bool FAppleTextToSpeech::IsSpeaking() const
{
	return IsActive() ? bIsSpeaking : false;
}

void FAppleTextToSpeech::StopSpeaking()
{
	if (IsActive())
	{
		if (IsSpeaking())
		{
			[SpeechSynthesizer stopSpeakingAtBoundary:AVSpeechBoundaryImmediate];
		}
		bIsSpeaking = false;
		UE_LOGF(LogTextToSpeech, Verbose, "Apple TTS stopped speaking.");
	}
}

float FAppleTextToSpeech::GetVolume() const
{
	return Volume;
}

void FAppleTextToSpeech::SetVolume(float InVolume)
{
	Volume = FMath::Clamp(InVolume, 0.f, 1.0f);
}

float FAppleTextToSpeech::GetRate() const
{
	return Rate;
}

void FAppleTextToSpeech::SetRate(float InRate)
{
	Rate = FMath::Clamp(InRate, 0.0f, 1.0f);
}

void FAppleTextToSpeech::Mute()
{
	if (IsActive() && !IsMuted())
	{
		SetMuted(true);
	}
}

void FAppleTextToSpeech::Unmute()
{
	if (IsActive() && IsMuted())
	{
		SetMuted(false);
	}
}

void FAppleTextToSpeech::OnActivated()
{
	ensureMsgf(!IsActive(), TEXT("Attempting to activate an already activated TTS. FTextToSpeechBase::Activate() should already guard against this."));
	TextToSpeechId IdCopy = GetId();
#if PLATFORM_MAC
	MainThreadCall(^{
#else // PLATFORM_IOS
	dispatch_async(dispatch_get_main_queue(), ^
	{
#endif
		SpeechSynthesizer = [[AVSpeechSynthesizer alloc] init];
		SpeechSynthesizerDelegate = [[FSpeechSynthesizerDelegate alloc] initWithOwningTextToSpeechId:IdCopy];
		SpeechSynthesizer.delegate = SpeechSynthesizerDelegate;

		// To get initial volume and rate, we need to retrieve it from an utterance
		AVSpeechUtterance* Utterance = [AVSpeechUtterance speechUtteranceWithString:@"Temp"];
		checkf(Utterance, TEXT("Failed to create an utterance for TTS."));
		Volume = Utterance.volume;
		Rate = Utterance.rate;
		ensure(Volume >= 0 && Volume <= 1.0f);
		ensure(Rate >= 0 && Rate <= 1.0f);

#if PLATFORM_IOS
		// This allows us to still hear the TTS when the IOS ringer is muted
		// SetFeature already takes care of keeping track of how many requests are made to activate/deactivate an audio feature
		[[IOSAppDelegate GetDelegate] SetFeature:EAudioFeature::Playback Active:true];
#endif
	});

	UE_LOGF(LogTextToSpeech, Verbose, "Apple TTS activated.");
}

void FAppleTextToSpeech::OnDeactivated()
{
	ensureMsgf(IsActive(), TEXT("Attempting to deactivate an already deactivated TTS. FTextToSpeechBase::Deactivate() should already guard against this."));
	// deallocate all AppKit objects in main thread just in case
#if PLATFORM_MAC
	MainThreadCall(^{
#else // PLATFORM_IOS
	dispatch_async(dispatch_get_main_queue(), ^
	{
#endif
		checkf(SpeechSynthesizerDelegate, TEXT("Deactivating Apple TTS with null speech synthesizer delegate. Speech synthesizer delegate must be valid throughout the lifetime of the object."));
		[SpeechSynthesizerDelegate release];
		checkf(SpeechSynthesizer, TEXT("Deactivating Apple TTS with null speech synthesizer. Speech synthesizer must be valid throughout the lifetime of the object."));
		[SpeechSynthesizer release];
	});
	UE_LOGF(LogTextToSpeech, Verbose, "Apple TTS deactivated.");

#if PLATFORM_IOS
		// SetFeature already takes care of keeping track of the number of requests to activate/deactivate a feature
		[[IOSAppDelegate GetDelegate] SetFeature:EAudioFeature::Playback Active:false];
#endif
}