// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"
#include "GenericPlatform/TextToSpeechBase.h"

@class AVSpeechSynthesizer;
@class FSpeechSynthesizerDelegate;
/**
* The text to speech implementation for Mac
*/
class 	FAppleTextToSpeech : public FTextToSpeechBase
{
public:
	FAppleTextToSpeech();
	virtual ~FAppleTextToSpeech();
	
	// FGenericTextToSpeech
	virtual void Speak(const FString& InStringToSpeak) override final;
	virtual bool IsSpeaking() const override final;
	virtual void StopSpeaking() override final;
	virtual float GetVolume() const override final;
	virtual void SetVolume(float InVolume) override final;
	virtual float GetRate() const override final;
	virtual void SetRate(float InRate) override final;
	virtual void Mute() override final;
	virtual void Unmute() override final;
protected:
	virtual void OnActivated() override final;
	virtual void OnDeactivated() override final;
	// ~
private:
	/**
	* True if the speech synthesizer is currently synthesizing any text, else false.
	* Speech synthesis for Mac is asynchronous, but we make the checking of whether something is speaking synchrnous
	* so as to simplify algorithms that depend on the check.
	* A precise check is not currently necessary.
	 */
 	bool bIsSpeaking;
	/** The current volume of the speech synthesizer. */
	float Volume;
	/** The current speech rate of the speech synthesizer */
	float Rate;
	/** The platform speech synthesizer that converts text to speech */
	AVSpeechSynthesizer* SpeechSynthesizer;
	/**
	* The delegate for the speech synthesizer. Callbacks indicating
	* speech synthesis playback progress, intruuption and completion are handled by this delegate.
	 */
	FSpeechSynthesizerDelegate* SpeechSynthesizerDelegate;
};