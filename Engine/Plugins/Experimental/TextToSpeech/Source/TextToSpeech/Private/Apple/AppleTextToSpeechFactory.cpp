// Copyright Epic Games, Inc. All Rights Reserved.
#include "Apple/AppleTextToSpeechFactory.h"
#include "GenericPlatform/ITextToSpeechFactory.h"
#include "Apple/AppleTextToSpeech.h"

TSharedRef<FTextToSpeechBase> FAppleTextToSpeechFactory::Create()
{
	return MakeShared<FAppleTextToSpeech>();
}
