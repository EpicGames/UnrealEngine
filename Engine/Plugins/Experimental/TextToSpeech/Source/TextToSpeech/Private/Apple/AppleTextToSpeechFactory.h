// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "CoreMinimal.h"
#include "GenericPlatform/ITextToSpeechFactory.h"

class FTextToSpeechBase;
/** The platform default text to speech factory for Apple. */
class FAppleTextToSpeechFactory : public ITextToSpeechFactory
{
public:
	FAppleTextToSpeechFactory() = default;
	virtual ~FAppleTextToSpeechFactory() = default;

	// ITextToSpeechFactory
	virtual TSharedRef<FTextToSpeechBase> Create() override;
	// ~
};
typedef FAppleTextToSpeechFactory FPlatformTextToSpeechFactory;
