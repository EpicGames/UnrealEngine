// Copyright Epic Games, Inc. All Rights Reserved.

#if USING_FLITE
#include "Flite/FliteAdapter.h"
#include "Flite/FliteSynthesizedSpeechData.h"
#include "TextToSpeechLog.h"
#include "Async/AsyncWork.h"
// Change to 1 to have sine tone data be written for debug
#define SINE_DEBUG 0
#if SINE_DEBUG
#include "DSP/SinOsc.h"
#endif
// Flite requires the use of certain windows headers
#if UE_FLITE_REQUIRES_WINDOWS_HEADERS
#include "Windows/AllowWindowsPlatformTypes.h"
#endif

extern "C"
{
// required to fix compile issue
#define __palmos__ 0
#include "flite.h"
	// Flite requires these forward declarations for registration of voices etc
	void usenglish_init(cst_voice *v);
	cst_lexicon *cmu_lex_init(void);
	// For now we only support RMS voice 
	cst_voice* register_cmu_us_rms(const char*voxdir);
	void unregister_cmu_us_rms(cst_voice* v);
} // extern "C"
#if UE_FLITE_REQUIRES_WINDOWS_HEADERS
#include "Windows/HideWindowsPlatformTypes.h"
#endif

FliteSpeechStreaming::FOnSynthesizedSpeechChunk FliteSpeechStreaming::OnSynthesizedSpeechChunk;
namespace PrivateFliteSpeechStreaming
{
	// written in game thread and read in audio render thread
	static std::atomic<bool> bShouldContinueStreaming(false);	
#if SINE_DEBUG
	Audio::FSineOsc SineOsc;
#endif
	/**
	 * Callback assigned to Flite when the library is
	 * synthesizing wave data. This is called everytime
	 * * a chunk of speech is done synthesizing.
	 * Called in background thread that synthesizes speech data.
	 */
	static int flite_audio_stream_chunk(const cst_wave* Wave, int ChunkStartSampleIndex, int NumSamples,
		int bIsLastChunk, cst_audio_streaming_info* AudioStreamingInfo)
	{
		if (!bShouldContinueStreaming)
		{
			UE_LOGF(LogTextToSpeech, VeryVerbose, "Stopping Flite speech streaming.");
			return CST_AUDIO_STREAM_STOP;
		}
		
		// Scratch buffer to convert PCM data for FLite to UE float format 
		TArray<float> PCMToFloatBuffer;
		PCMToFloatBuffer.AddZeroed(NumSamples);
		// Note it's possible for a stream of size 0 to come in with it being the last chunk
		for (int32 CurrentIndex = 0; CurrentIndex < NumSamples; ++CurrentIndex)
		{
			// Flite wave samples are stored as shorts (PCM) we need to convert to float
			// The magic number is the conversion from PCM to float
			PCMToFloatBuffer[CurrentIndex] = static_cast<float>(Wave->samples[ChunkStartSampleIndex + CurrentIndex]) / 32768.0f;
#if SINE_DEBUG
			// Overwriting speech data with sine wave data
			float Sample = SineOsc.ProcessAudio() * 0.5f;
			PCMToFloatBuffer[CurrentIndex] = Sample;
#endif
		}
		FFliteSynthesizedSpeechData SpeechData(MoveTemp(PCMToFloatBuffer), static_cast<int32>(Wave->sample_rate), static_cast<int32>(Wave->num_channels));
		int ReturnCode = CST_AUDIO_STREAM_CONT;
		if (bIsLastChunk == 1)
		{
			UE_LOGF(LogTextToSpeech, VeryVerbose, "Synthesizing last speech chunk of %i bytes.", SpeechData.GetNumSpeechSamples());
			bShouldContinueStreaming = false;
			SpeechData.bIsLastChunk = true;
			ReturnCode = CST_AUDIO_STREAM_STOP;
		}
		// The delegate should ALWAYS be bound at this point. It's an error to stream without binding this delegate
		ensure(FliteSpeechStreaming::OnSynthesizedSpeechChunk.IsBound());
		FliteSpeechStreaming::OnSynthesizedSpeechChunk.ExecuteIfBound(MoveTemp(SpeechData));
		return ReturnCode;
	}
} // namespace PrivateFliteSpeechStreaming

/** Task to synthesize speech data from a given text in a background thread */
class FAsyncFliteSpeechSynthesisWorker : public FNonAbandonableTask
{
	friend class FAutoDeleteAsyncTask<FAsyncFliteSpeechSynthesisWorker>;
protected:
	FFliteAdapter& FliteAdapter;
	FString SynthesisText;
public:
	FAsyncFliteSpeechSynthesisWorker(FFliteAdapter& InFliteAdapter, FString InSynthesisText)
		: FliteAdapter(InFliteAdapter)
		, SynthesisText(MoveTemp(InSynthesisText))
	{
		
	}

	/**
	 * Performs the async text to speech synthesis
	 */
	void DoWork()
	{
		UE_LOGF(LogTextToSpeech, VeryVerbose, "Sy Background thread synthesizing %ls.", *SynthesisText);
		FliteAdapter.SynthesizeSpeechData_AnyThread(SynthesisText);
	}

	FORCEINLINE TStatId GetStatId() const
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(FAsyncFliteSpeechSynthesisWorker, STATGROUP_ThreadPoolAsyncTasks);
	}
};

typedef FAutoDeleteAsyncTask<FAsyncFliteSpeechSynthesisWorker> FAsyncFliteSpeechSynthesis;

FFliteAdapter::FFliteAdapter()
	: FliteCurrentVoice(nullptr)
	, FliteAudioStreamingInfo(nullptr)
	, FliteRate(1.0f)
{
	// Must be called before any of the Flite API can be called
	flite_init();
	// setup voices with audio streaming
	FliteAudioStreamingInfo = new_audio_streaming_info();
	// THis is the ideal size for now as any bigger will result in pops and clicks during playback
	// as the submix listener can't catch up 
	FliteAudioStreamingInfo->min_buffsize = 1024;
	FliteAudioStreamingInfo->asc = PrivateFliteSpeechStreaming::flite_audio_stream_chunk;
	// Always register voices after creating the streaming info
	RegisterVoices();
	
#if SINE_DEBUG
	// @TODOAccessibility: Just hardcoding the sample rate for now. Should match that of the voice 
	PrivateFliteSpeechStreaming::SineOsc.Init(16000, 440, 0.5f);
#endif
}

FFliteAdapter::~FFliteAdapter()
{
	StopSynthesizeSpeechData_GameThread();
	UnregisterVoices();
}

void FFliteAdapter::RegisterVoices()
{
	// the streaming info should be created first before we register any of the voices
	check(FliteAudioStreamingInfo);
	FliteCurrentVoice = register_cmu_us_rms(nullptr);
	// unable to create voice 
	check(FliteCurrentVoice);
	feat_set(FliteCurrentVoice->features, "streaming_info", audio_streaming_info_val(FliteAudioStreamingInfo));
}

void FFliteAdapter::UnregisterVoices()
{
	// We should always have a valid voice at thsi point.
	check(FliteCurrentVoice);
	// This seems to already clean up the audio streaming info  
	unregister_cmu_us_rms(FliteCurrentVoice);
	FliteCurrentVoice = nullptr;
	FliteAudioStreamingInfo = nullptr;
}

void FFliteAdapter::SynthesizeSpeechData_AnyThread(const FString& InText)
{
	UE_LOGF(LogTextToSpeech, VeryVerbose, "Starting speech synthesis in background thread.");
	// Flite can only accept ANSI chars
	auto TextSource = StringCast<ANSICHAR>(*InText);
	const ANSICHAR* CharTextPtr = TextSource.Get();
	// we MUST have a valid voice at this point for the selected language
	check(FliteCurrentVoice);
	// This starts synthesizing the text
	// This in turns calls the callback bound in the audio streaming info
	flite_text_to_wave(CharTextPtr, FliteCurrentVoice);
}

void FFliteAdapter::StartSynthesizeSpeechData_GameThread(const FString& InText)
{
	UE_LOGF(LogTextToSpeech, VeryVerbose, "Toggling streaming flag on.");
	PrivateFliteSpeechStreaming::bShouldContinueStreaming = true;
	(new FAsyncFliteSpeechSynthesis(*this, InText))->StartBackgroundTask();
}

void FFliteAdapter::StopSynthesizeSpeechData_GameThread()
{
	UE_LOGF(LogTextToSpeech, VeryVerbose, "Toggling speech streaming flag off.");
	PrivateFliteSpeechStreaming::bShouldContinueStreaming = false;
}

// Flite uses a multiplier to stretch the duratin of the speech 
// Thus speaking at 1/3 duration makes it speak 3x faster 
// Speaking at 2x duration makes it speak half as fast etc 
// We put a cap on the max stretch to minimize memory footprint as longer stretch means more audio chunks queued
const float FFliteAdapter::FliteMaximumDurationStretch = 1.5f;
const float FFliteAdapter::FliteMinimumDurationStretch = 1.0f / 3.0f;

float FFliteAdapter::GetFliteMaximumRate() const
{
	// The duration stretch is a multiplier for how long an utterance takes to speak 
	// Thus the inverse of the duration stretch is the speech rate 
	// Consequently, duration stretch of < 1.0 means the utterance is spoken faster. Thus inverse of the minimum stretch duration is the maximum speech rate 
	return 1.0f / FFliteAdapter::FliteMinimumDurationStretch;
}

float FFliteAdapter::GetFliteMinimumRate() const
{
	// see GetFliteMaximumSpeechRate()
	return 1.0f / FFliteAdapter::FliteMaximumDurationStretch;
}

float FFliteAdapter::GetRate_GameThread() const
{
	check(IsInGameThread());
	float UnrealRate = (FliteRate - GetFliteMinimumRate()) / (GetFliteMaximumRate() - GetFliteMinimumRate());
	ensure(UnrealRate >= 0.0f && UnrealRate <= 1.0f);
	return UnrealRate;
}

void FFliteAdapter::SetRate_GameThread(float InRate)
{
	check(IsInGameThread());
	ensure(InRate >= 0.0f && InRate <= 1.0f);
	FliteRate = GetFliteMinimumRate() + (InRate * (GetFliteMaximumRate() - GetFliteMinimumRate()));
	ensure(FliteRate >= GetFliteMinimumRate() && FliteRate <= GetFliteMaximumRate());
	if (FliteCurrentVoice)
	{
		// To convert from rate to duration stretch, we take the inverse of the rate  
		feat_set_float(FliteCurrentVoice->features, "duration_stretch", (1.0f / FliteRate));
	}
}
#endif
