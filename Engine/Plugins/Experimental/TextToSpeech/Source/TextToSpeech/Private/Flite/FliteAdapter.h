// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#if USING_FLITE
#include "Delegates/Delegate.h"

class FString;
struct FFliteSynthesizedSpeechData;

struct FliteSpeechStreaming
{
	/** Delegate that fires when flite streams in a chunk of audio */
	DECLARE_DELEGATE_OneParam(FOnSynthesizedSpeechChunk, FFliteSynthesizedSpeechData);
	static FOnSynthesizedSpeechChunk OnSynthesizedSpeechChunk;
};

struct cst_voice_struct;
struct cst_audio_streaming_info_struct;

/**
* Adapter that handles Flite initialization and deinitialization as well as converting audio data from flite format to unreal format
*/
class FFliteAdapter
{
	public:
	FFliteAdapter();
	~FFliteAdapter();
	/** Starts synthesizing the given text into speech. Must be called from the game thread.*/
	void StartSynthesizeSpeechData_GameThread(const FString& InText);
	/** Stops the synthesis of the current text */
	void StopSynthesizeSpeechData_GameThread();
	/** Called in background threads to */
	void SynthesizeSpeechData_AnyThread(const FString& InText);
	
	// The TTS interface accepts values between 0.0f and 1.0f
	// We convert the UE rate to Flite rate and vice versa here 
	/** 
	* Gets the speech rate Flite is synthesizing text in Unreal units. 
	* * Value guaranteed to be between 0.0f and 1.0f 
	*/
	float GetRate_GameThread() const;
	/** 
	* Sets the speech rate for Flite to synthesize text at.
	* User expected to pass in a value between 0.0f and 1.0f
	*/
	void SetRate_GameThread(float InRate);
	
private:
	/** Returns the maximum speech rate allowed in Flite units */
	float GetFliteMaximumRate() const;
	/** Returns the minimum speech rate allowed in Flite units */
	float GetFliteMinimumRate() const;
	/** Register all desired voices for Flite */
	void RegisterVoices();
	/** Unregister all voices used from Flite */
	void UnregisterVoices();
	/** The currently active voice. Note only US_EN supported right now */
	cst_voice_struct* FliteCurrentVoice;
	/** The streaming info for the current voice being used */
	cst_audio_streaming_info_struct* FliteAudioStreamingInfo;
	/** The speech rate the voice should be synthesized at in Flite units */
	float FliteRate;
	// Flite uses duration stretch to elongate how long it takes to speak an utternace
	// E.g 2x duration stretch means it takes 2x as long to speak an utterance
	// This can also be interpreted as speaking 0.5x speed 
	/**
	* Flite uses the feasture "duration_stretch" t scale the duration of an utterance.o 
	* E.g Duration stretch of 2.0 means the utterance will take twice as long to finish speaking.
	* Thus a duration_stretch of X can be interpreted as speaking at a rate of 1/X
	*/
	static const float FliteMaximumDurationStretch;
	static const float FliteMinimumDurationStretch;
};
#endif
