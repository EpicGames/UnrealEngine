// Copyright Epic Games, Inc. All Rights Reserved.

#if USING_FLITE
#include "HAL/LowLevelMemTracker.h"
#include "HAL/UnrealMemory.h"
THIRD_PARTY_INCLUDES_START
extern "C"
{
#include "cst_alloc.h" 
} // extern "C"

/**
* These overrrides will use the Unreal implementations of malloc etc to override the Flite implementations
* which will just use malloc.h
* See Flite/Flite-<version>/src/utils/cst_alloc.c for more details
*/

extern "C"
{
	void* cst_safe_alloc(int size)
	{
		// The original implementation of Flite's malloc uses calloc so we are following it's implementation here 
		return cst_safe_calloc(size);
	}

	void* cst_safe_calloc(int size)
	{
		LLM_SCOPE(ELLMTag::UI);
		if (size >= 0)
		{
		// following Flite's implementation detail for malloc(0) 
			if (size == 0)
			{
				++size;
			}
			void* Ptr = FMemory::Malloc(size);
			FMemory::Memzero(Ptr, size);
			return Ptr;
		}
		return nullptr;
	}

	void* cst_safe_realloc(void* p, int size)
	{
		LLM_SCOPE(ELLMTag::UI);
		return FMemory::Realloc(p, size);
	}

	void cst_free(void* p)
	{
		FMemory::Free(p);
	}
} // extern "C"

THIRD_PARTY_INCLUDES_END
#endif // USING_FLITE
