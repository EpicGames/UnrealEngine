// Copyright Epic Games, Inc. All Rights Reserved.

#if USING_FLITE
#include "Flite/FliteSynthesizedSpeechData.h"

FFliteSynthesizedSpeechData::FFliteSynthesizedSpeechData()
	: SampleRate(INDEX_NONE)
	, NumChannels(INDEX_NONE)
	, bIsLastChunk(false)
{
	
}

FFliteSynthesizedSpeechData::FFliteSynthesizedSpeechData(TArray<float> InSpeechBuffer, int32 InSampleRate, int32 InNumChannels)
	: SpeechBuffer(MoveTemp(InSpeechBuffer))
	, SampleRate(InSampleRate)
	, NumChannels(InNumChannels)
	, bIsLastChunk(false)
{
	
}

bool FFliteSynthesizedSpeechData::IsValid() const
{
	return NumChannels > 0
		&& NumChannels < 3
		&& SampleRate > 0;
}

void FFliteSynthesizedSpeechData::Reset()
{
	SpeechBuffer.Reset();
	SampleRate = INDEX_NONE;
	NumChannels = INDEX_NONE;
}

#endif