// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#if USING_FLITE
#include "Containers/Array.h"

struct FScriptContainerElement;

struct FFliteSynthesizedSpeechData
{
	FFliteSynthesizedSpeechData();
	// TArrays to be passed by move construction
	FFliteSynthesizedSpeechData(TArray<float> InSpeechBuffer, int32 InSampleRate, int32 InNumChannels);
	bool HasData() const { return SpeechBuffer.Num() > 0;}
	bool IsValid() const;
	void Reset();
	int32 GetNumSpeechSamples() const { return SpeechBuffer.Num(); }
	float* GetSpeechData() { return SpeechBuffer.GetData(); }
	bool IsLastChunk() const { return bIsLastChunk; }
	
	TArray<float> SpeechBuffer;
	int32 SampleRate;
	int32 NumChannels;
	bool bIsLastChunk;
};
#endif
