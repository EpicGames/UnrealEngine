// Copyright Epic Games, Inc. All Rights Reserved.

#if USING_FLITE
#include "Flite/FliteTextToSpeech.h"
#include "Flite/FliteAdapter.h"
#include "Flite/FliteSynthesizedSpeechData.h"
#include "Flite/FliteTextToSpeechSubmixListener.h"
#include "AudioDevice.h"
#include "TextToSpeechLog.h"


/**
 * Flite TTS Overview
 * There are 2 components of performing TTS via the Flite library
 * 1. Flite Adapter - A wrapper for the Flite library. Given an FString, this handles synthesizing and streaming in of the speech data along with controlling the speech rate
 * 2. Flite Submix Listener - Handles queuing and playback of the synthesized speech data
 * The speech synthesis and final playback occurs across 3 threads
 * 1. Game Thread - Request for starting and stopping speech synthesis
 * 2. Background Thread - Where the audio data gets synthesized by the library and streamed in
 * 3. Audio Render Thread - Where the audio data synthesized from the background thread gets played
 */
FFliteTextToSpeech::FFliteTextToSpeech()
{
}

FFliteTextToSpeech::~FFliteTextToSpeech()
{
	Deactivate();
}

void FFliteTextToSpeech::Speak(const FString& InStringToSpeak)
{
	check(IsInGameThread());
	if (IsActive())
	{
		if (IsSpeaking())
		{
			StopSpeaking();
		}
		UE_LOGF(LogTextToSpeech, VeryVerbose, "Starting to synthesize text: %ls", *InStringToSpeak);
		FliteAdapter->StartSynthesizeSpeechData_GameThread(InStringToSpeak);
		TTSSubmixListener->StartPlayback_GameThread();
	}
}

bool FFliteTextToSpeech::IsSpeaking() const
{
	check(IsInGameThread());
	if (IsActive())
	{
		return TTSSubmixListener->IsPlaybackActive();
	}
	return false;
}

void FFliteTextToSpeech::StopSpeaking()
{
	check(IsInGameThread());
	if (IsActive())
	{
		UE_LOGF(LogTextToSpeech, VeryVerbose, "Stopping speech synthesis.");
		FliteAdapter->StopSynthesizeSpeechData_GameThread();
		if (IsSpeaking())
		{
			TTSSubmixListener->StopPlayback_GameThread();
		}
	}
}

float FFliteTextToSpeech::GetVolume() const
{
	check(IsInGameThread());
	if (IsActive())
	{
		return TTSSubmixListener->GetVolume();
	}
	return 0.0f;
}

void FFliteTextToSpeech::SetVolume(float InVolume)
{
	check(IsInGameThread());
	if (IsActive())
	{
		float ClampedVolume = FMath::Clamp(InVolume, 0.0f, 1.0f);
		TTSSubmixListener->SetVolume(ClampedVolume);
	}
}

float FFliteTextToSpeech::GetRate() const
{
	check(IsInGameThread());
	if (IsActive())
	{
		return FliteAdapter->GetRate_GameThread();
	}
	return 0.0f;
}

void FFliteTextToSpeech::SetRate(float InRate)
{
	check(IsInGameThread());
	if (IsActive())
	{
		float ClampedRate = FMath::Clamp(InRate, 0.0f, 1.0f);
		FliteAdapter->SetRate_GameThread(ClampedRate);
	}

}

void FFliteTextToSpeech::Mute()
{
	check(IsInGameThread());
	if (IsActive())
	{
		SetMuted(true);
		TTSSubmixListener->Mute();
}
}

void FFliteTextToSpeech::Unmute()
{
	check(IsInGameThread());
	if (IsActive())
	{
		SetMuted(false);
		TTSSubmixListener->Unmute();
	}
}

void FFliteTextToSpeech::OnActivated()
{
	check(IsInGameThread());
	UE_LOGF(LogTextToSpeech, Verbose, "Activating speech synthesis");
	FliteSpeechStreaming::OnSynthesizedSpeechChunk.BindRaw(this, &FFliteTextToSpeech::OnSynthesizedSpeechChunk_AnyThread);
	if (FAudioDeviceManager* AudioDeviceManager = FAudioDeviceManager::Get())
	{
		if (FAudioDeviceHandle AudioDeviceHandle = AudioDeviceManager->GetMainAudioDeviceHandle())
		{
			if (FAudioDevice* AudioDevice = AudioDeviceHandle.GetAudioDevice())
			{
				FliteAdapter = MakeUnique<FFliteAdapter>();
				TTSSubmixListener = MakeShared<FFliteTextToSpeechSubmixListener>(GetId());
				AudioDeviceHandle->RegisterSubmixBufferListener(TTSSubmixListener->AsShared(), AudioDevice->GetMainSubmixObject());
				// RegisterSubmixBufferListener lazily enqueues the registration on the audio thread,
			// so we have to wait for the audio thread to destroy.
				FAudioCommandFence Fence;
				Fence.BeginFence();
				Fence.Wait();
			}
		}
	}
}

void FFliteTextToSpeech::OnDeactivated()
{
	check(IsInGameThread());
	UE_LOGF(LogTextToSpeech, Verbose, "Deactivating text to speech.");
	if (IsSpeaking())
	{
		StopSpeaking();
	}
	if (FliteSpeechStreaming::OnSynthesizedSpeechChunk.IsBound())
	{
		FliteSpeechStreaming::OnSynthesizedSpeechChunk.Unbind();
	}
	if (FAudioDeviceManager* AudioDeviceManager = FAudioDeviceManager::Get())
	{
		if (FAudioDeviceHandle AudioDeviceHandle = AudioDeviceManager->GetMainAudioDeviceHandle())
		{
			AudioDeviceHandle->UnregisterSubmixBufferListener(TTSSubmixListener->AsShared(), AudioDeviceHandle->GetMainSubmixObject());
			FAudioCommandFence Fence;
			Fence.BeginFence();
			Fence.Wait();
		}
		FliteAdapter = nullptr;
		TTSSubmixListener.Reset();
	}
}

void FFliteTextToSpeech::OnSynthesizedSpeechChunk_AnyThread(FFliteSynthesizedSpeechData InSynthesizedSpeechData)
{
	if (IsActive())
	{
		TTSSubmixListener->QueueSynthesizedSpeechChunk_AnyThread(MoveTemp(InSynthesizedSpeechData));
}
}

#endif
