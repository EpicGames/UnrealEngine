// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#if USING_FLITE
#include "GenericPlatform/TextToSpeechBase.h"

struct FFliteSynthesizedSpeechData;
class FFliteTextToSpeechSubmixListener;

class FFliteTextToSpeech : public FTextToSpeechBase
{
public:
	FFliteTextToSpeech();
	virtual ~FFliteTextToSpeech();
	
	// FTextToSpeechBase
	virtual void Speak(const FString& InStringToSpeak) override final;
	virtual bool IsSpeaking() const override final;
	virtual void StopSpeaking() override final;
	virtual float GetVolume() const override final;
	virtual void SetVolume(float InVolume) override final;
	virtual float GetRate() const override final;
	virtual void SetRate(float InRate) override final;
	virtual void Mute() override final;
	virtual void Unmute() override final;
protected:
	virtual void OnActivated() override final;
	virtual void OnDeactivated() override final;
	// ~ FTextToSpeechBase
private:
	/**
	* Callback to queue synthesized audio data into the submix listener for playback
	* Should only be called by worker threads in background threads 
	*/
	void OnSynthesizedSpeechChunk_AnyThread(FFliteSynthesizedSpeechData InSynthesizedSpeechData);

	/**
	* Handles synthesizing of speech and returning speech audio
	* data in a format our Audio Engine can use
	*/
	TUniquePtr<class FFliteAdapter> FliteAdapter;
	/** Handles routing synthesized speech audio data directly to the hardware */
	TSharedPtr<FFliteTextToSpeechSubmixListener, ESPMode::ThreadSafe> TTSSubmixListener;
};
#endif
