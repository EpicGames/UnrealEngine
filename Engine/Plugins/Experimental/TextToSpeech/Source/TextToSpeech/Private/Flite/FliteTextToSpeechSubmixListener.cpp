// Copyright Epic Games, Inc. All Rights Reserved.

#if USING_FLITE
#include "Flite/FliteTextToSpeechSubmixListener.h"
#include "AudioDevice.h"
#include "Flite/FliteSynthesizedSpeechData.h"
#include "TextToSpeechLog.h"

/** Defined in FTextToSpeechBase.cpp */
extern TMap<TextToSpeechId, TWeakPtr<FTextToSpeechBase>> ActiveTextToSpeechMap;

FFliteTextToSpeechSubmixListener::FFliteTextToSpeechSubmixListener(TextToSpeechId InOwningTTSId)
	: OwningTTSId(InOwningTTSId)
	, Volume(1.0f)
	, bAllowPlayback(false)
	, bMuted(false)
{
	// we must always have a valid owning TTS provided 
	check(InOwningTTSId != FTextToSpeechBase::InvalidTextToSpeechId);
	// Sample Rate Conversion ratio is 1.0f for now and the input from Flite is always mono
	// we assume that all Flite synthesis is mono (1 channel)
	AudioResampler.Init(Audio::EResamplingMethod::Linear, 1.0f, 1);
}

FFliteTextToSpeechSubmixListener::~FFliteTextToSpeechSubmixListener()
{
	StopPlayback_GameThread();
}

const FString& FFliteTextToSpeechSubmixListener::GetListenerName() const
{
	static const FString ListenerName = TEXT("Flite Text-To-Speech Listener");
	return ListenerName;
}

void FFliteTextToSpeechSubmixListener::OnNewSubmixBuffer(const USoundSubmix* OwningSubmix, float* AudioData, int32 NumSamples, int32 NumChannels, const int32 SampleRate, double AudioClock)
{
	if (!bAllowPlayback)
	{
		return;
	}
	const int32 NumFrames = NumSamples / NumChannels;
	OutputSpeechBuffer.Reset();
	OutputSpeechBuffer.AddZeroed(NumFrames);
	// flag to detect if we're playing the last chunk of a speech utterance
	bool bPlayedLastChunk = false;
	int32 NumPoppedSamples = 0;
	{
		FScopeLock Lock(&SynthesizedSpeechBufferCS);
		if (SynthesizedSpeechBuffer.IsEmpty())
		{
			return;
		}
		NumPoppedSamples = SynthesizedSpeechBuffer.PopData_AudioRenderThread(OutputSpeechBuffer, NumFrames, bPlayedLastChunk);
	}
	// Note it's possible for the number of popped samples to be 0
	// The way that flite streaming works, we could have have no samples popped but an indication that 
	// the last chunk was synthesized 

	// If speech is muted, we want the speech to continue playing and pick up when we unmute
	// Thus set volume to 0 for when muted 
	float VolumeMultiplier = bMuted ? 0.0f : Volume.load();
	ensure(VolumeMultiplier >= 0 && VolumeMultiplier <= 1.0);
	// We only mix in up to the number of popped samples as the remainder of the output speech buffer is silence
	// output buffer is mono. Thus the frame index is the same
	// as the index to read data from the output speech buffer
	for (int32 FrameIndex = 0; FrameIndex < NumPoppedSamples; ++FrameIndex)
	{
		// we only mix into first 2 channels for L and R
		for (int32 ChannelIndex = 0; ChannelIndex < 2; ++ChannelIndex)
		{
			AudioData[FrameIndex * NumChannels + ChannelIndex] += OutputSpeechBuffer[FrameIndex] * VolumeMultiplier;
		}
	}
	if (bPlayedLastChunk)
	{
		UE_LOGF(LogTextToSpeech, VeryVerbose, "Finishing playing last chunk of audio.");
		TextToSpeechId IdCopy = OwningTTSId;
		const TMap<TextToSpeechId, TWeakPtr<FTextToSpeechBase>>& ActiveTextToSpeechMapRef = ActiveTextToSpeechMap;
		FFunctionGraphTask::CreateAndDispatchWhenReady([IdCopy, ActiveTextToSpeechMapRef]()
			{
				const TWeakPtr<FTextToSpeechBase>* TTSPtr = ActiveTextToSpeechMapRef.Find(IdCopy);
				if (TTSPtr && TTSPtr->IsValid())
				{
					TTSPtr->Pin()->OnTextToSpeechFinishSpeaking_GameThread();
				}
			}, TStatId(), NULL, ENamedThreads::GameThread);
		bAllowPlayback = false;
	}
}

bool FFliteTextToSpeechSubmixListener::IsRenderingAudio() const
{
	return bAllowPlayback;
}

void FFliteTextToSpeechSubmixListener::StartPlayback_GameThread()
{
	ensure(IsInGameThread());
	bAllowPlayback = true;
}

void FFliteTextToSpeechSubmixListener::QueueSynthesizedSpeechChunk_AnyThread(FFliteSynthesizedSpeechData InSynthesizedSpeechChunk)
{
	// this is called from a background thread from speech synth streaming
	if (FAudioDeviceManager* AudioDeviceManager = FAudioDeviceManager::Get())
	{
		if (FAudioDeviceHandle AudioDeviceHandle = AudioDeviceManager->GetMainAudioDeviceHandle())
		{
			const int32 SampleRate = AudioDeviceHandle->GetSampleRate();
			// We first upsample the speech data to meet the audio engine sample rate
			float SampleRateConversionRatio = static_cast<float>(SampleRate) / static_cast<float>(InSynthesizedSpeechChunk.SampleRate);
			UE_LOGF(LogTextToSpeech, VeryVerbose, "Sample rate conversion ratio is : %f.", SampleRateConversionRatio);
			AudioResampler.SetSampleRateRatio(SampleRateConversionRatio);
			// Prepare SRC buffer to be of the appropriate size given the SRC factor
			// If pitching down, this buffer will be larger than input buffer
			// If pitching up, this buffer will be smaller than input buffer
			int32 NumConvertedSamples = InSynthesizedSpeechChunk.SampleRate / SampleRateConversionRatio;
			int32 OutputSamples = INDEX_NONE;
			TArray<float> SampleRateConversionBuffer;
			SampleRateConversionBuffer.AddZeroed(NumConvertedSamples);
			// Perform the sample rate conversion
			int32 ErrorCode = AudioResampler.ProcessAudio(InSynthesizedSpeechChunk.GetSpeechData(), InSynthesizedSpeechChunk.GetNumSpeechSamples(), InSynthesizedSpeechChunk.IsLastChunk(), SampleRateConversionBuffer.GetData(), SampleRateConversionBuffer.Num(), OutputSamples);
			ensure(OutputSamples <= NumConvertedSamples);
			if (ErrorCode != 0)
			{
				UE_LOGF(LogTextToSpeech, Warning, "Problem occured resampling Flite speech data.");
				ensure(false);
			}
			// due to resampler implementation details, it's possible to have the number of output samples not match up to the number of requested samples. We shrink the SRC buffer in that case
			UE_LOGF(LogTextToSpeech, VeryVerbose, "Calculated SRC buffer size: %i. Actual output samples: %i.", NumConvertedSamples, OutputSamples);
			SampleRateConversionBuffer.SetNum(OutputSamples);
			// @TODOAccessibility: Refactor. Terrible to just overwrite the data and break encapsulation
			InSynthesizedSpeechChunk.SpeechBuffer = MoveTemp(SampleRateConversionBuffer);
			{
				FScopeLock Lock(&SynthesizedSpeechBufferCS);
				SynthesizedSpeechBuffer.PushData(MoveTemp(InSynthesizedSpeechChunk));
			}
			return;
		}
	}
	// we should always have the main audio device. If we get here, something's really wrong
	ensure(false);
}
	
void FFliteTextToSpeechSubmixListener::StopPlayback_GameThread()
{
	check(IsInGameThread());
	bAllowPlayback = false;
	// empties the speech buffer immediately
	{
		FScopeLock Lock(&SynthesizedSpeechBufferCS);
		SynthesizedSpeechBuffer.Reset_GameThread();
	}
	UE_LOGF(LogTextToSpeech, VeryVerbose, "Emptied speech buffer.");
}

bool FFliteTextToSpeechSubmixListener::IsPlaybackActive() const
{
	return bAllowPlayback;
}

float FFliteTextToSpeechSubmixListener::GetVolume() const
{
	ensure(IsInGameThread());
	if (bMuted)
	{
		return 0;
	}
	return Volume.load();
}

void FFliteTextToSpeechSubmixListener::SetVolume(float InVolume)
{
	ensure(IsInGameThread());
	ensure(InVolume >= 0 && InVolume <= 1.0f);
	Volume.store(InVolume);
}

void FFliteTextToSpeechSubmixListener::Mute()
{
	bMuted = true;
}

void FFliteTextToSpeechSubmixListener::Unmute()
{
	bMuted = false;
}

/**
* Controls the capacity of the FSynthesizedSpeechBuffer. The larger the capacity of the buffer, the more synthesiazed audio data that can be held by 
*/
static int32 gFliteSpeechBufferCapacity = 512;
static FAutoConsoleVariableRef FliteSpeechBufferCapacityRef(
	TEXT("TextToSpeech.Flite.SpeechBufferCapacity"),
	gFliteSpeechBufferCapacity,
	TEXT("Controls the capacity of the Flite synthesized speechbuffer upon initialization of the Flite text to speech submix listener. The buffer will hold chunks of streamed, synthesized speech data. The larger the size the capacity of the buffer, the more audio data can be synthesized at once for playback and the more memory the buffer will take up. It's recommeneded to keep the value greater than 256 and less than 1024.")
);
FFliteTextToSpeechSubmixListener::FSynthesizedSpeechBuffer::FSynthesizedSpeechBuffer()
	: SynthesizedSpeechChunks(gFliteSpeechBufferCapacity)
	, ReadIndex(0)
	, WriteIndex(0)
, ChunkReadIndex(0)
{
	
}

void FFliteTextToSpeechSubmixListener::FSynthesizedSpeechBuffer::PushData(FFliteSynthesizedSpeechData InSpeechData)
{
	if (IsFull())
	{
		// We ran out of space
		UE_LOGF(LogTextToSpeech, Verbose, "Flite audio buffer ran out of space. ");
		ensure(false);
	}
	SynthesizedSpeechChunks[WriteIndex] = MoveTemp(InSpeechData);
	WriteIndex = SynthesizedSpeechChunks.GetNextIndex(WriteIndex);
}

int32 FFliteTextToSpeechSubmixListener::FSynthesizedSpeechBuffer::PopData_AudioRenderThread(TArray<float>& OutBuffer, int32 NumRequestedSamples, bool& bOutPoppingLastChunk)
{
	UE_LOGF(LogTextToSpeech, VeryVerbose, "Popping speech data. %i samples requested.", NumRequestedSamples);
	bOutPoppingLastChunk = false;
	if (IsEmpty() || (NumRequestedSamples < 1))
	{
		UE_LOGF(LogTextToSpeech, VeryVerbose, "SPeech buffer is empty. No data popped.");
		return NumRequestedSamples;
	}
	int32 NumPoppedSamples = 0;
	while (NumPoppedSamples < NumRequestedSamples)
	{
		// num popped samples should be strictly smaller than requested samples
		check(NumPoppedSamples < NumRequestedSamples);
		FFliteSynthesizedSpeechData& CurrentChunk = SynthesizedSpeechChunks[ReadIndex];
		const int32 NumRemainingRequestedSamples = NumRequestedSamples - NumPoppedSamples;
		const int32 NumRemainingSamplesInChunk = CurrentChunk.GetNumSpeechSamples() - ChunkReadIndex;

		// @TODOAccessibility: Refactor
		if (NumRemainingSamplesInChunk == 0)
		{
// because of the way flite streaming works
			// We can get a chunk that's empty but is the last chunk  
			ChunkReadIndex = 0;
			ReadIndex = SynthesizedSpeechChunks.GetNextIndex(ReadIndex);
			const bool bIsLastChunk = CurrentChunk.IsLastChunk();
			CurrentChunk.Reset();
			// Early out if we finished playing a last chunk of a speech synth
			if (bIsLastChunk)
			{
				bOutPoppingLastChunk = true;
				break;
			}
		}
		else if (NumRemainingRequestedSamples >= NumRemainingSamplesInChunk)
		{
			// we can just memcopy all the remaining data over and move on to the next chunk
			FMemory::Memcpy(&OutBuffer[NumPoppedSamples], &CurrentChunk.GetSpeechData()[ChunkReadIndex], NumRemainingSamplesInChunk * sizeof(float));
			NumPoppedSamples += NumRemainingSamplesInChunk;
			ensure(NumPoppedSamples <= NumRequestedSamples && NumPoppedSamples > 0);
			ChunkReadIndex = 0;
			ReadIndex = SynthesizedSpeechChunks.GetNextIndex(ReadIndex);
			const bool bIsLastChunk = CurrentChunk.IsLastChunk();
			CurrentChunk.Reset();
			// Early out if we finished playing a last chunk of a speech synth
			if (bIsLastChunk)
			{
				bOutPoppingLastChunk = true;
				break;
			}
		}
		else
		{
			// we just memcopy over a portion of the chunk
			FMemory::Memcpy(&OutBuffer[NumPoppedSamples], &CurrentChunk.GetSpeechData()[ChunkReadIndex], NumRemainingRequestedSamples * sizeof(float));
			NumPoppedSamples += NumRemainingRequestedSamples;
			ensure(NumPoppedSamples <= NumRequestedSamples && NumPoppedSamples > 0);
			ChunkReadIndex += NumRemainingRequestedSamples;
		}
		// if we're out of chunks, early out even if we still have requested samples
		if (IsEmpty())
		{
			break;
		}
	}
	UE_LOGF(LogTextToSpeech, VeryVerbose, "Number of samples popped: %i", NumPoppedSamples);
	ensure(NumPoppedSamples <= NumRequestedSamples && NumPoppedSamples > 0);
	return NumPoppedSamples;
}

// this should only be called from the audio render thread or game thread
void FFliteTextToSpeechSubmixListener::FSynthesizedSpeechBuffer::Reset_GameThread()
{
	ReadIndex = WriteIndex;
	ChunkReadIndex = 0;
}

bool FFliteTextToSpeechSubmixListener::FSynthesizedSpeechBuffer::IsEmpty() const
{
	return ReadIndex == WriteIndex;;
}

const FFliteSynthesizedSpeechData& FFliteTextToSpeechSubmixListener::FSynthesizedSpeechBuffer::GetCurrentChunk_AudioRenderThread() const
{
	return SynthesizedSpeechChunks[ReadIndex];
}

bool FFliteTextToSpeechSubmixListener::FSynthesizedSpeechBuffer::IsFull() const
{
	// if the next index from write index is the read index
	// the buffer is full.
	return SynthesizedSpeechChunks.GetNextIndex(WriteIndex) == ReadIndex;
}

#endif
