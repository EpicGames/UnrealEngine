// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#if USING_FLITE
#include "Containers/CircularBuffer.h"
#include "AudioResampler.h"
#include "GenericPlatform/TextToSpeechBase.h"
#include "ISubmixBufferListener.h"

struct FFliteSynthesizedSpeechData;


/** Handles storing TTS audio data to be played. Expects audio data as floats  */
class FFliteTextToSpeechSubmixListener : public ISubmixBufferListener
{
public:
	FFliteTextToSpeechSubmixListener(TextToSpeechId InOwningTTSId);
	virtual ~FFliteTextToSpeechSubmixListener();
	
	/** Starts playback and allows any queued audio datat to be played */
	void StartPlayback_GameThread();
	/**
	* * Queues a chunk of synthesized audio streamed from the flite library. 
	* Called by background worker threads. 
	*/
	void QueueSynthesizedSpeechChunk_AnyThread(FFliteSynthesizedSpeechData InSynthesizedSpeechChunk);
	/** Stops playback and stops all audio from playing */
	void StopPlayback_GameThread();
	/** Returns true if the submix listener is playing the queued speech audio data. Else false */
	bool IsPlaybackActive() const;
	/** Mutes the submix listener. No speech audio data will be played */
	void Mute();
	/** Unmutes the submix listener. Speech audio data will be audible again */
	void Unmute();
	/** Returns true if the submix listener is muted. Else false. */
	bool IsMuted() const { return bMuted; }
	/** Returns the volume the speech data is played at */
	float GetVolume() const;
	/** Sets the volume the speech data should be played at */
	void SetVolume(float InVolume);

	// ISubmixBufferListener
	virtual const FString& GetListenerName() const override;
	virtual void OnNewSubmixBuffer(const USoundSubmix* OwningSubmix, float* AudioData, int32 NumSamples, int32 NumChannels, const int32 SampleRate, double AudioClock) override;
	virtual bool IsRenderingAudio() const override;
	// ~ISubmixBufferListener
private:
	class FSynthesizedSpeechBuffer
	{
	public:
		FSynthesizedSpeechBuffer();
		/** Should only be called in background thread */
		void PushData(FFliteSynthesizedSpeechData InSpeechData);
		/** 
		* Should only be called from audio render thread 
		* Expects a zeroed out buffer. 
		* Note that it is possible for the buffer to not be empty but pop 0 data from a chunk
		* due to Flite's occassional streaming of a 0 data chunk with a flag indicating that the chunk 
		* is finished synthesizing. 
		*/
		int32 PopData_AudioRenderThread(TArray<float>& OutBuffer, int32 NumRequestedSamples, bool& bOutPoppingLastChunk);
		/** Should only be called from game thread */
		void Reset_GameThread();
		bool IsEmpty() const;
		/** Should only be called from audio render thread */
		const FFliteSynthesizedSpeechData& GetCurrentChunk_AudioRenderThread() const;
	private:
		bool IsFull() const;
		TCircularBuffer<FFliteSynthesizedSpeechData> SynthesizedSpeechChunks;
		int32 ReadIndex;
		int32 WriteIndex;
		// Used to keep track of where in the current chunk we should start reading from
		// Modified in audio render thread
		int32 ChunkReadIndex;
	};
	FSynthesizedSpeechBuffer SynthesizedSpeechBuffer;
	Audio::FResampler AudioResampler;
	FCriticalSection SynthesizedSpeechBufferCS;
	/**
	* The buffer that stores the data with DSP applied
	* should only be used in audio render thread
	*/
	TArray<float> OutputSpeechBuffer;
/** ID for the TTS object htat owns this usbmix listener */
	TextToSpeechId OwningTTSId;

	/** 
	* Current volume between 0.0f and 1.0f. 
	* written/read in game thread and read in audio render thread
	*/
	std::atomic<float> Volume;
	/** 
	* If true, this allows audio playback of the queued audio data 
	* written in game thread read in audio render thread
	*/
	std::atomic<bool> bAllowPlayback;
	/**
	* If true, this mutes the submix listener and prevents any audio from playing 
	* written in game thread and read in audio render thread
	*/
	std::atomic<bool> bMuted;
};
#endif
