// Copyright Epic Games, Inc. All Rights Reserved.

#include "GenericPlatform/TextToSpeechBase.h"

/** 
* A map contianing all currently alive text to speech objects. Created TTS are registered to this map on activation and unregistered on deactivation.
* Used to ensure a TTS object is still alive when OnTextToSpeechFinishSpeaking_GameThread() is called from othere threads
*/
TMap<TextToSpeechId, TWeakPtr<FTextToSpeechBase>> ActiveTextToSpeechMap;

FTextToSpeechBase::FTextToSpeechBase()
	: bMuted(false)
	, bActive(false)
{
	static TextToSpeechId RuntimeIdCounter = 0;
	if (RuntimeIdCounter == TNumericLimits<TextToSpeechId>::Max())
	{
		RuntimeIdCounter = TNumericLimits<TextToSpeechId>::Min();
	}
	if (RuntimeIdCounter == InvalidTextToSpeechId)
	{
		++RuntimeIdCounter;
	}
	Id = RuntimeIdCounter++;
}

FTextToSpeechBase::~FTextToSpeechBase()
{
	if (TextToSpeechFinishSpeakingDelegate.IsBound())
	{
		TextToSpeechFinishSpeakingDelegate.Unbind();
	}
	Deactivate();
	Id = InvalidTextToSpeechId;
}

void FTextToSpeechBase::OnTextToSpeechFinishSpeaking_GameThread()
{
	check(IsInGameThread());
	TextToSpeechFinishSpeakingDelegate.ExecuteIfBound();
}

void FTextToSpeechBase::Activate()
{
	if (!IsActive())
	{
		// we register with the map 
		ActiveTextToSpeechMap.FindOrAdd(Id, AsShared());
		OnActivated();
		bActive = true;
	}
}

void FTextToSpeechBase::Deactivate()
{
	if (IsActive())
	{
		OnDeactivated();
		bActive = false;
		// unregistering this from the map to avoid child classes from triggering the on finish speaking delegate
		// from other threads when the TTS has already been destroyed 
		ActiveTextToSpeechMap.Remove(Id);
	}
}

