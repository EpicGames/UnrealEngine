// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
// We're doing this as the TextToSpeechModule.cpp is using COMPILED_PLATFORM_HEADER
// which doesn't have a clean way to override and point to a shared platform.
#include "Apple/AppleTextToSpeechFactory.h"