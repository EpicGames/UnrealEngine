// Copyright Epic Games, Inc. All Rights Reserved.

#if PLATFORM_LINUX
#include "Linux/LinuxTextToSpeechFactory.h"
#include "Flite/FliteTextToSpeech.h"

TSharedRef<FTextToSpeechBase> FLinuxTextToSpeechFactory::Create()
{
	return MakeShared<FFliteTextToSpeech>();
}
#endif
