// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#if PLATFORM_LINUX
#include "GenericPlatform/ITextToSpeechFactory.h"

class FTextToSpeechBase;
/** The platform default text to speech factory for Linux platforms. */
class FLinuxTextToSpeechFactory : public ITextToSpeechFactory
{
public:
	FLinuxTextToSpeechFactory() = default;
	virtual ~FLinuxTextToSpeechFactory() = default;

	virtual TSharedRef<FTextToSpeechBase> Create() override;
};

typedef FLinuxTextToSpeechFactory FPlatformTextToSpeechFactory;
#endif
