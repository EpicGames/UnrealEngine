// Copyright Epic Games, Inc. All Rights Reserved.

#include "TextToSpeechEngineSubsystem.h"
#include "TextToSpeechModule.h"
#include "GenericPlatform/ITextToSpeechFactory.h"
#include "GenericPlatform/TextToSpeechBase.h"
#include "TextToSpeechLog.h"

UTextToSpeechEngineSubsystem::UTextToSpeechEngineSubsystem()
{

}

UTextToSpeechEngineSubsystem::~UTextToSpeechEngineSubsystem()
{
	RemoveAllChannels();
}

void UTextToSpeechEngineSubsystem::Initialize(FSubsystemCollectionBase & Collection)
{
	Super::Initialize(Collection);
	// @TODOAccessibility: Parse a config file or something to populate the channel map with pre-defined channels and settings etc 
}

void UTextToSpeechEngineSubsystem::Deinitialize()
{
	RemoveAllChannels();
	Super::Deinitialize();
}

void UTextToSpeechEngineSubsystem::SpeakOnChannel(FName InChannelId, UPARAM(ref, DisplayName = "String To Speak") const FString& InStringToSpeak)
{
	TSharedRef<FTextToSpeechBase>* TTS = ChannelIdToTextToSpeechMap.Find(InChannelId);
	if (TTS)
	{
		(*TTS)->Speak(InStringToSpeak);
	}
}

void UTextToSpeechEngineSubsystem::StopSpeakingOnChannel(UPARAM(DisplayName = "Channel Id") FName InChannelId)
{
	TSharedRef<FTextToSpeechBase>* TTS = ChannelIdToTextToSpeechMap.Find(InChannelId);
	if (TTS)
	{
		return (*TTS)->StopSpeaking();
	}
	// @TODOAccessibility: Warning about invlaid channel id etc 
}

void UTextToSpeechEngineSubsystem::StopSpeakingOnAllChannels()
{
	for (TPair<FName, TSharedRef<FTextToSpeechBase>> Channel : ChannelIdToTextToSpeechMap)
	{
		Channel.Value->StopSpeaking();
}
}

bool UTextToSpeechEngineSubsystem::IsSpeakingOnChannel(UPARAM(DisplayName = "Channel Id") FName InChannelId) const
{
	const TSharedRef<FTextToSpeechBase>* TTS = ChannelIdToTextToSpeechMap.Find(InChannelId);
	if (TTS)
	{
		return (*TTS)->IsSpeaking();
	}
	return false;
}

float UTextToSpeechEngineSubsystem::GetVolumeOnChannel(UPARAM(DisplayName = "Channel Id") FName InChannelId)
{
	const TSharedRef<FTextToSpeechBase>* TTS = ChannelIdToTextToSpeechMap.Find(InChannelId);
	if (TTS)
	{
		return (*TTS)->GetVolume();
	}
	return 0.0f;
}

void UTextToSpeechEngineSubsystem::SetVolumeOnChannel(UPARAM(DisplayName = "Channel Id") FName InChannelId, UPARAM(DisplayName = "Volume") float InVolume)
{
	TSharedRef<FTextToSpeechBase>* TTS = ChannelIdToTextToSpeechMap.Find(InChannelId);
	if (TTS)
	{
		(*TTS)->SetVolume(InVolume);
	}
}

float UTextToSpeechEngineSubsystem::GetRateOnChannel(UPARAM(DisplayName = "Channel Id") FName InChannelId) const
{
	const TSharedRef<FTextToSpeechBase>* TTS = ChannelIdToTextToSpeechMap.Find(InChannelId);
	if (TTS)
	{
		return (*TTS)->GetRate();
	}
	return 0.0f;
}

void UTextToSpeechEngineSubsystem::SetRateOnChannel(UPARAM(DisplayName = "Channel Id") FName InChannelId, UPARAM(DisplayName = "Rate") float InRate)
{
	const TSharedRef<FTextToSpeechBase>* TTS = ChannelIdToTextToSpeechMap.Find(InChannelId);
	if (TTS)
	{
		(*TTS)->SetRate(InRate);
	}
}

void UTextToSpeechEngineSubsystem::MuteChannel(UPARAM(DisplayName = "Channel Id") FName InChannelId)
{
	TSharedRef<FTextToSpeechBase>* TTS = ChannelIdToTextToSpeechMap.Find(InChannelId);
	if (TTS)
	{
		(*TTS)->Mute();
	}
}

void UTextToSpeechEngineSubsystem::UnmuteChannel(UPARAM(DisplayName = "Channel Id") FName InChannelId)
{
	TSharedRef<FTextToSpeechBase>* TTS = ChannelIdToTextToSpeechMap.Find(InChannelId);
	if (TTS)
	{
		(*TTS)->Unmute();
	}
}

bool UTextToSpeechEngineSubsystem::IsChannelMuted(UPARAM(DisplayName = "Channel Id") FName InChannelId) const
{
	const TSharedRef<FTextToSpeechBase>* TTS = ChannelIdToTextToSpeechMap.Find(InChannelId);
	if (TTS)
	{
		return (*TTS)->IsMuted();
	}
	return false;
}

void UTextToSpeechEngineSubsystem::ActivateChannel(UPARAM(DisplayName = "Channel Id") FName InChannelId)
{
	TSharedRef<FTextToSpeechBase>* TTS = ChannelIdToTextToSpeechMap.Find(InChannelId);
	if (TTS)
	{
		(*TTS)->Activate();
	}
}

void UTextToSpeechEngineSubsystem::ActivateAllChannels()
{
	for (TPair < FName, TSharedRef<FTextToSpeechBase>>& Channel : ChannelIdToTextToSpeechMap)
	{
		Channel.Value->Activate();
	}
}

void UTextToSpeechEngineSubsystem::DeactivateChannel(UPARAM(DisplayName = "Channel Id") FName InChannelId)
{
	TSharedRef<FTextToSpeechBase>* TTS = ChannelIdToTextToSpeechMap.Find(InChannelId);
	if (TTS)
	{
		(*TTS)->Deactivate();
	}
}

void UTextToSpeechEngineSubsystem::DeactivateAllChannels()
{
	for (TPair < FName, TSharedRef<FTextToSpeechBase>>& Channel : ChannelIdToTextToSpeechMap)
	{
		Channel.Value->Deactivate();
	}
}

bool UTextToSpeechEngineSubsystem::IsChannelActive(UPARAM(DisplayName = "Channel Id") FName InChannelId) const
{
	const TSharedRef<FTextToSpeechBase>* TTS = ChannelIdToTextToSpeechMap.Find(InChannelId);
	if (TTS)
	{
		return (*TTS)->IsActive();
	}
	return false;
}

void UTextToSpeechEngineSubsystem::AddDefaultChannel(UPARAM(DisplayName = "New Channel Id") FName InNewChannelId)
{
	if (!DoesChannelExist(InNewChannelId))
	{
		ITextToSpeechModule& Module = ITextToSpeechModule::Get();
		// platform factory always guaranteed to be valid 
		ChannelIdToTextToSpeechMap.Add(InNewChannelId, Module.GetPlatformFactory()->Create());
	}
	else
	{
		UE_LOGF(LogTextToSpeech, Warning, "The requested channel Id: %ls already exists.", *InNewChannelId.ToString());
	}
}

void UTextToSpeechEngineSubsystem::AddCustomChannel(UPARAM(DisplayName = "New Channel Id") FName InNewChannelId)
{
	if (!DoesChannelExist(InNewChannelId))
	{
		ITextToSpeechModule& Module = ITextToSpeechModule::Get();
		if (TSharedPtr<ITextToSpeechFactory> CustomFactory = Module.GetCustomFactory())
		{
			ChannelIdToTextToSpeechMap.Add(InNewChannelId, CustomFactory->Create());
		}
		else
		{
			UE_LOGF(LogTextToSpeech, Warning, "Requested using custom text to speech, but no custom text to speech factory was set.");
		}
	}
	else
	{
		UE_LOGF(LogTextToSpeech, Warning, "The requested channel Id: %ls already exists.", *InNewChannelId.ToString());
	}
}

void UTextToSpeechEngineSubsystem::RemoveChannel(UPARAM(DisplayName = "Channel Id") FName InChannelId)
{
	TSharedRef<FTextToSpeechBase>* TTS = ChannelIdToTextToSpeechMap.Find(InChannelId);
	if (TTS)
	{
		(*TTS)->Deactivate();
		ChannelIdToTextToSpeechMap.Remove(InChannelId);
	}
	else
	{
		UE_LOGF(LogTextToSpeech, Warning, "Channel Id: %ls does not exist. Cannot be removed.", *InChannelId.ToString());
	}
}

void UTextToSpeechEngineSubsystem::RemoveAllChannels()
{
	DeactivateAllChannels();
	ChannelIdToTextToSpeechMap.Empty();
}

bool UTextToSpeechEngineSubsystem::DoesChannelExist(UPARAM(DisplayName = "Channel Id") FName InChannelId) const
{
	return ChannelIdToTextToSpeechMap.Contains(InChannelId);
}

int32 UTextToSpeechEngineSubsystem::GetNumChannels() const
{
	return ChannelIdToTextToSpeechMap.Num();
}