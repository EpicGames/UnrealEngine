// Copyright Epic Games, Inc. All Rights Reserved.


#include "TextToSpeechModule.h"
#include "TextToSpeechLog.h"
#include COMPILED_PLATFORM_HEADER(TextToSpeechFactory.h)

class FTextToSpeechModule : public ITextToSpeechModule
{
public:
	FTextToSpeechModule() = default;
	virtual ~FTextToSpeechModule() = default;
	// ITextToSpeechModule 
	virtual TSharedPtr<ITextToSpeechFactory> GetPlatformFactory() const override
	{
		return PlatformFactory;
	}
	virtual TSharedPtr<ITextToSpeechFactory> GetCustomFactory() const override
	{
		return CustomFactory;
	}
	virtual void SetCustomFactory(const TSharedRef<ITextToSpeechFactory>& InFactory) override
	{
		CustomFactory = InFactory;
	}
	// ~

	// IModuleInterface
	virtual void StartupModule() override
	{
		PlatformFactory = MakeShared<FPlatformTextToSpeechFactory>();
	}

	virtual void ShutdownModule() override
	{

	}
	// ~
private:
	TSharedPtr<ITextToSpeechFactory> PlatformFactory;
	TSharedPtr<ITextToSpeechFactory> CustomFactory;
};

DEFINE_LOG_CATEGORY(LogTextToSpeech);
IMPLEMENT_MODULE(FTextToSpeechModule, TextToSpeech)
