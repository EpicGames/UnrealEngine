// Copyright Epic Games, Inc. All Rights Reserved.

#if PLATFORM_WINDOWS
#include "Windows/WindowsTextToSpeechFactory.h"
#include "GenericPlatform/ITextToSpeechFactory.h"
#include "GenericPlatform/TextToSpeechBase.h"
#include "Flite/FliteTextToSpeech.h"

TSharedRef<FTextToSpeechBase> FWindowsTextToSpeechFactory::Create()
{
	return MakeShared<FFliteTextToSpeech>();
}

#endif
