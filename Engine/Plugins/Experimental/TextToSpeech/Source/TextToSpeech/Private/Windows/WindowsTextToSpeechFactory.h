// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#if PLATFORM_WINDOWS
#include "CoreMinimal.h"
#include "GenericPlatform/ITextToSpeechFactory.h"

class FTextToSpeechBase;
/** The platform default text to speech factory for Windows. */
class FWindowsTextToSpeechFactory : public ITextToSpeechFactory
{
public:
	FWindowsTextToSpeechFactory() = default;
	virtual ~FWindowsTextToSpeechFactory() = default;

	virtual TSharedRef<FTextToSpeechBase> Create() override;
};
typedef FWindowsTextToSpeechFactory FPlatformTextToSpeechFactory;
#endif

