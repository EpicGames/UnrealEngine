// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"

class FTextToSpeechBase;

/**
 * Abstract base class for all factories that create text to speech objects.
 * Platform default factories have already been provided for all allowed platforms for this plugin.
 * Inherit from this class and register it with the text to speech module to create your own custom text to speech objects
* See ITextToSpeechModule.h for more details. 
 */
class TEXTTOSPEECH_API ITextToSpeechFactory
{
public:
	virtual ~ITextToSpeechFactory() = default;
	/**
	 * Creates an instance of a text to speech object. 
	 * It is advised to only have 1 TTS as multiple TTS can speak over each other.
	 * Callers are responsible for managing the lifetime of the TTS
	 */
	virtual TSharedRef<FTextToSpeechBase> Create() = 0;
};

