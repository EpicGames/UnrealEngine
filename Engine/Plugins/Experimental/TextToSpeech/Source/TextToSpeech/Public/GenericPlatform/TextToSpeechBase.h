// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

typedef int8 TextToSpeechId;
/**
 * Abstract base class for all text to speech engines.
 * Subclass for each new speech synthesis method to be used.
 * Text to speech takes in an FString and synthesizes audio data based on the text using the underlying speech synthesizer.
 */
class TEXTTOSPEECH_API FTextToSpeechBase : public TSharedFromThis<FTextToSpeechBase>
{
public:
	/** A delegate that fires when the underlying text to speech has successfully finished speaking the requested string */
	DECLARE_DELEGATE(FOnTextToSpeechFinishSpeaking);
	static const TextToSpeechId InvalidTextToSpeechId = -1;
	
	FTextToSpeechBase();
	virtual ~FTextToSpeechBase();
	/** Returns the ID associated with this text to speech object */
	TextToSpeechId GetId() const { return Id; }
	/**
	 * This should only be called by text to speech implementations to signal that 
	 * a requessted string has successfully finished speaking.
	 * Note that this should not be invoked when the spoken string is interrupted or stopped.
	 */
	void OnTextToSpeechFinishSpeaking_GameThread();
	/** Sets the TextToSpeechFinishedSpeakingDelegate */
	void SetTextToSpeechFinishedSpeakingDelegate(const FOnTextToSpeechFinishSpeaking& Delegate) { TextToSpeechFinishSpeakingDelegate = Delegate; }
	/**
	 * Activates the TTS system and does the necessary set up.
	 * Calls OnActivated() at the end of set up.
	 * Override OnActivated() in child classes to provide additional set up.
	 */
	void Activate();
	
	/**
	 * Deactivates the TTS and performs tear down.
	 * OnDeactivated() called at the start of tear down.
	 * Override OnDeactivated() in child classes to provide additional tear down.
	 */
	void Deactivate();

	/** Returns true if the TTS is muted. Else false. */
	bool IsMuted() const { return bMuted; }

	/** Returns true if the TTS is active. Else false. */
	bool IsActive() const { return bActive; }
	/**
	 * Immediately speaks the requested string.
	 * If another string is being spoken by the TTS, the current string will immediately be stopped
	 * before the requested string is spoken.
	 * Note: Passing in long sentences or paragraphs are not recommended. Split the paragraph into shorter sentences and 
	 * register with the OnTextToSpeechFinishSpeakingDelegate to pass in the next chunk. 
	 */
	virtual void Speak(const FString& InStringToSpeak) = 0;
	/** Returns true if any string is currently being spoken, else false. */
	virtual bool IsSpeaking() const = 0;
	/** Immediately stops speaking the current string being spoken */
	virtual void StopSpeaking() = 0;
	/** Returns the current volume being spoken at.  Value is between 0 and 1.0f. */
	virtual float GetVolume() const { return 0.0f; }
	/** Sets the volume for strings to be spoken at. Clamps to 0 and 1.0f. */
	virtual void SetVolume(float InVolume) {}
	
	/** Returns the current speech rate strings are spoken at. Value is between 0 and 1.0f.*/
	virtual float GetRate() const { return 0.0f; }
	/** Sets the current speech rate strings should be spoken at. Clamps to 0 and 1.0f. */
	virtual void SetRate(float InRate) {}
	
	/** Mutes the TTS so no synthesized speech is audible */
	virtual void Mute() {}
	/** Unmutes the TTS so requests to speak strings are audible again. */
	virtual void Unmute() {}

protected:
	/** Sets the flag for whether the TTS is muted */
	void SetMuted(bool bInMuted) { bMuted = bInMuted; }
	/**
	 * Called at the end of Activate()
	 * Override in child classes to provide additional set up.
	 */
	virtual void OnActivated() {}
	/**
	 * Called at the start of Deactivate()
	 * Override in child classes to provide additional tear down
	 */
	virtual void OnDeactivated() {}		
private:
	/**
	 * The delegate that is called when the TTS successfully finishes speaking a string.
	 * This should not be invoked when speech is interrupted or stopped.
	 */
	FOnTextToSpeechFinishSpeaking TextToSpeechFinishSpeakingDelegate;

	/** The runtime ID for this TTS object. Used primarily for retrieving the TTS object during multithreaded calls */
	TextToSpeechId Id;
	/** True if the TTS is muted. Else false. */
	bool bMuted;
	/** True if the TTS is active. Elsxe false */
	bool bActive;
};

