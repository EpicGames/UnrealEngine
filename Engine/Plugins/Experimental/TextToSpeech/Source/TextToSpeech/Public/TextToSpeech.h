// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
/** 
 * Convenience header to get access to all text to speech functionality.
 * For creation of text to speech objects, see TextToSpeechModule.h
 * For usage of text to speech objects, see TextToSpeechBase.h
 */
#include "GenericPlatform/TextToSpeechBase.h"
#include "TextToSpeechModule.h"
#include "GenericPlatform/ITextToSpeechFactory.h"
#include "TextToSpeechEngineSubsystem.h"