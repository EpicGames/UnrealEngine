// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"
#include "Subsystems/EngineSubsystem.h"
#include "TextToSpeechEngineSubsystem.generated.h"


class FTextToSpeechBase;

/**
* Subsystem for interacting with the text to speech system via blueprints.
* The subsystem consists of a number of text to speech channels associated with a text to speech channel Id.
* Each text to speech channel provides text to speech functionality such as vocalizing a string, stopping a vocalized string, controlling the audio settings of the channel and muting/unmuting a channel. 
* Most of the functions take a text to speech channel Id which allows you to request text to speech functionality on the requested channel.
* All newly added text to speech channels start off deactivated and must be activated before any text to speech functionality can be used.
*/
UCLASS()
class TEXTTOSPEECH_API UTextToSpeechEngineSubsystem: public UEngineSubsystem
{
	GENERATED_BODY()
public:
	UTextToSpeechEngineSubsystem();
	virtual ~UTextToSpeechEngineSubsystem();

	/**
	* Immediately vocalizes the requested string asynchronously on the requested text to speech channel, interrupting any string that is already being vocalized on the channel.
	* If the provided channel Id does not exist, nothing will be vocalized.
	* Before executing this function, a text to speech channel must be added and activated.
	* To create a platform default text to speech channel, use AddDefaultChannel.
	* To create a custom text to speech channel with a user-implemented C++ text to speech class, use AddCustomChannel.
	* This function is not intended for long strings that span multiple sentences or paragraphs.
	* @param InChannelId The Id of the channel to speak on.
	* @param InStringToSpeak The string to speak on the requested channel.
	* @see AddDefaultChannel, AddCustomChannel, ActivateChannel, ActivateAllChannels
	*/
	UFUNCTION(BlueprintCallable, Category=TextToSpeech)
	void SpeakOnChannel(UPARAM(DisplayName = "Channel Id") FName InChannelId, UPARAM(ref, DisplayName="String To Speak") const FString& InStringToSpeak);

	/** 
	* Immediately stops any currently vocalized string on the channel.
	* If the provided channel Id does not exist, nothing will happen.
	* @param InChannelId The Id of the channel speech should be stopped on. 
	*/
	UFUNCTION(BlueprintCallable, Category = TextToSpeech)
	void StopSpeakingOnChannel(UPARAM(DisplayName = "Channel Id") FName InChannelId);

	/** Immediately stops strings from being vocalized on all text to speech channels. */
	UFUNCTION(BlueprintCallable, Category=TextToSpeech)
	void StopSpeakingOnAllChannels();

	/** 
	* Return true when the targeted text to speech channel is vocalising, otherwise false.
	* @param InChannelId The id of the channel to check if a string is being vocalized.
	*/
	UFUNCTION(BlueprintCallable, Category=TextToSpeech)
	bool IsSpeakingOnChannel(UPARAM(DisplayName = "Channel Id") FName InChannelId) const;

	/** 
	* Returns the current volume strings are vocalized on a text to speech channel.  Value is between 0.0f and 1.0f.
	* If the provided channel Id doesn't exist, 0.0f will be returned. 
	* @param InChannelId The Id for the channel to retrieve the volume from.
	*/
	UFUNCTION(BlueprintCallable, Category = TextToSpeech)
	float GetVolumeOnChannel(UPARAM(DisplayName = "Channel Id") FName InChannelId);

	/** 
	* Sets the volume for strings vocalized on a text to speech channel. 
	* If the provided channel Id does not exist, nothing will happen.
	* @param InChannelId The Id for the channel to set for.
	* @param InVolume The volume to be set on the channel. The value will be clamped between 0.0f and 1.0f. 
	*/
	UFUNCTION(BlueprintCallable, Category = TextToSpeech)
	void SetVolumeOnChannel(UPARAM(DisplayName = "Channel Id") FName InChannelId, UPARAM(DisplayName = "Volume") float InVolume);

	/** 
	* Returns the current speech rate strings are vocalized on a text to speech channel. Value is between 0.0 and 1.0.
	* If the provided channel Id does not exist, 0.0f will be returned. 
	* @param InChannelId The Id for the channel to get the speech rate from. 
	*/
	UFUNCTION(BlueprintCallable, Category = TextToSpeech)
	float GetRateOnChannel(UPARAM(DisplayName = "Channel Id") FName InChannelId) const;

	/** 
	* Sets the current speech rate strings should be vocalized on a text to speech channel.
	* If the provided channel does not exist, nothing will happen. 
	* @param InChannelId The Id for the channel to set the speech rate on.
	* @param InRate The speech rate to set for the channel. Value will be clamped between 0.0f and 1.0f.
	*/
	UFUNCTION(BlueprintCallable, Category = TextToSpeech)
	void SetRateOnChannel(UPARAM(DisplayName = "Channel Id") FName InChannelId, UPARAM(DisplayName = "Rate") float InRate);

	/** 
	* Mutes a text to speech channel so no vocalized strings are audible on that channel.
	* If the requested channel is already muted, nothing will happen.
	* @param InChannelId The Id for the channel to mute.
	*/
	UFUNCTION(BlueprintCallable, Category = TextToSpeech)
	void MuteChannel(UPARAM(DisplayName = "Channel Id") FName InChannelId);

	/** 
	* Unmutes a text to speech channel so vocalized strings are audible on the channel.
	* If the requested channel is already unmuted, nothing will happen.
	@param InChannelId The Id for the channel to unmute.
	*/
	UFUNCTION(BlueprintCallable, Category = TextToSpeech)
	void UnmuteChannel(UPARAM(DisplayName = "Channel Id") FName InChannelId);

	/** 
	* Returns true if the text to speech channel is muted, otherwise false.
	* @param InChannelId The Id for the channel to check if it is muted.
	*/
	UFUNCTION(BlueprintCallable, Category = TextToSpeech)
	bool IsChannelMuted(UPARAM(DisplayName = "Channel Id") FName InChannelId) const;

	/**
	* Activates a text to speech channel to accept requests to perform text to speech functionality.
	* If the provided channel Id does not exist, nothing will happen. 
	* @param InChannelId The Id of the channel to activate. 
	*/
	UFUNCTION(BlueprintCallable, Category = TextToSpeech)
	void ActivateChannel(UPARAM(DisplayName="Channel Id") FName InChannelId);

	/** Activates all text to speech channels to accept requests for text to speech functionality. */
	UFUNCTION(BlueprintCallable, Category = TextToSpeech)
	void ActivateAllChannels();

	/**
	* Deactivates a text to speech channel and stops any vocalized strings on that channel. Future Requests for text to speech functionality will do nothing.
	* If the provided channel Id does not exist, nothing will happen. 
	* @param InChannelId The Id for the channel to deactivate.
	*/
	UFUNCTION(BlueprintCallable, Category=TextToSpeech)
	void DeactivateChannel(UPARAM(DisplayName = "Channel Id") FName InChannelId);

	/** Deactivates all text to speech channels making all requests for text to speech functionality do nothing. */
	UFUNCTION(BlueprintCallable, Category = TextToSpeech)
	void DeactivateAllChannels();

	/** 
	* Returns true if the text to speech channel is active, otherwise false.
	* @param InChannelId The Id for the channel to check if it is active. 
	*/
	UFUNCTION(BlueprintCallable, Category = TextToSpeech)
	bool IsChannelActive(UPARAM(DisplayName = "Channel Id") FName InChannelId) const;

	/**
	* Creates a new  channel for text to speech requests to be made to a platform C++ text to speech class. 
	* This will not create the channel if the provided channel id is not unique.
	* Newly added channels must be activated to use text to speech functionalities.
	* For out-of-the-box text to speech support, this is most likely the channel creation method you want.
	* @param InChannelId The Id of the channel you want to add
	* @see ActivateChannel, ActivateAllChannels
	*/
	UFUNCTION(BlueprintCallable, Category=TextToSpeech)
	void AddDefaultChannel(UPARAM(DisplayName = "New Channel Id") FName InNewChannelId);

	/** 
	* Creates a new text to speech channel where text to speech requests are fulfilled by a user implemented C++ text to speech class. 
	* If you have not specified a custom text to speech class to be used, use AddDefaultChannel instead.
	* This will not add a channel if the channel Id is not unique or if the user has not specified a custom text to speech class to be used in ITextToSpeechModule.
	* Newly added channels must be activated to use text to speech functionalities.
	* @see ITextToSpeechModule, AddDefaultChannel, ActivateChannel, ActivateAllChannels
	*/
	UFUNCTION(BlueprintCallable, Category = TextToSpeech)
	void AddCustomChannel(UPARAM(DisplayName = "New Channel Id") FName InNewChannelId);

	/** 
	* Removes a text to speech channel, preventing all further requests for text to speech functionality from the channel. 
	* If the provided channel Id does not exist, nothing will happen.
	* @param InChannelId The Id for the channel you want removed. 
	*/
	UFUNCTION(BlueprintCallable, Category=TextToSpeech)
	void RemoveChannel(UPARAM(DisplayName = "Channel Id") FName InChannelId);

	/** Removes all text to speech channels, preventin future requests for text to speech functionality on all channels. */ 
	UFUNCTION(BlueprintCallable, Category = TextToSpeech)
	void RemoveAllChannels();

	/**
	* Returns true if a text to speech channel associated with a channel Id exists. Otherwise, the function returns false.
	* @param InChannelId The channel Id to test if it is associated with a channel.
	*/
	UFUNCTION(BlueprintCallable, Category=TextToSpeech)
	bool DoesChannelExist(UPARAM(DisplayName = "Channel Id") FName InChannelId) const;

	/** Returns the number of text to speech channels that have been added. */
	UFUNCTION(BlueprintCallable, Category=TextToSpeech, meta=(DisplayName="Get Number Of Channels"))
	int32 GetNumChannels() const;

// UEngineSubsystem
	virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	virtual void Deinitialize() override;
	// ~
private:
	/** The map of channel Ids to native text to speech objects. */
	TMap<FName, TSharedRef<FTextToSpeechBase>> ChannelIdToTextToSpeechMap;
};
