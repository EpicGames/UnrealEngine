// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Modules/ModuleManager.h"
#include "Modules/ModuleInterface.h"


class ITextToSpeechFactory;

/**
*  The module for text to speech. 
* Access this class via the singleton Get() convenience function.
* Use this class to get and set text to speech factories for creating text to speech objects.
* A platform default text to speech factory will be created for all allowed platforms for this plugin.
* Users can subclass from ITextToSpeechFactory and set their own custom factory if desired.
* Samples:
* TSharedRef<FTextToSpeechBase> MyPlatformTTS = ITextToSpeechModule::Get().GetPlatformFactory()->Create();
* TSharedRef<FMyTextToSpeechFactory> MyTTSFactory = MakeShared<FMyTextToSpeechFactory>();
* ITextToSpeechModule::Get().SetCustomFactory(MyTTSFactory);
* TSharedRef<FTextToSpeechBase> MyCustomTTS = ITextToSharedModule::Get().GetCustomFactory()->Create();
*/
class TEXTTOSPEECH_API ITextToSpeechModule : public IModuleInterface
{
public:
	/** Get this module and load it if required. */
	static ITextToSpeechModule& Get()
	{
		static const FName ModuleName = "TextToSpeech";
		return FModuleManager::Get().LoadModuleChecked<ITextToSpeechModule>(ModuleName);
	}
	/**
	*  Returns the factory that creates the platform default text to speech object.
	* For all allowed platforms, this is guaranteed to be valid. 
	*/
	virtual TSharedPtr<ITextToSpeechFactory> GetPlatformFactory() const = 0;
	/**
	* Returns the user set text to speech factory.
	* This can be null if no user text to speech factory has been set.
	* It's up to the user to check the returned factory is valid. 
	*/
	virtual TSharedPtr<ITextToSpeechFactory> GetCustomFactory() const = 0;
	/**
	* Sets a user text to speech factory.
	* Use this to register custom text to speech factories for creating custom 
	* text to speech objects. 
	* @see ITextToSpeechFactory
	*/
	virtual void SetCustomFactory(const TSharedRef<ITextToSpeechFactory>& InFactory) = 0;
};

