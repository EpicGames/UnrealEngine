// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;
using UnrealBuildTool;

public class TextToSpeech : ModuleRules
{

	// platforms that inherit from this should override and return true if they intend to use Flite
	protected virtual bool bIsFliteUsed
	{
		get
		{
			// Expand to other platforms as needed
			return Target.IsInPlatformGroup(UnrealPlatformGroup.Windows) 
				|| Target.IsInPlatformGroup(UnrealPlatformGroup.Android)
				|| Target.IsInPlatformGroup(UnrealPlatformGroup.Unix);
		}
	}

	public TextToSpeech(ReadOnlyTargetRules Target) : base(Target)
	{
		bRequiresPlatformSDK = true;
		
		PublicDependencyModuleNames.AddRange(
			new string[] 
			{
				"Core",
				"CoreUObject"
			}
		);
	
		PublicIncludePathModuleNames.AddRange(
			new string[] 
			{
			// For access to platform includes to support TTS
			// E.g Mac and IOS TTS requires Apple's frameworks
				"ApplicationCore"
			}
		);
		
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Engine"
			}
		);
		if (bIsFliteUsed)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[] 
				{
					"AudioMixer",
					"SignalProcessing",
					"AudioPlatformConfiguration"
				}
			);
			AddEngineThirdPartyPrivateStaticDependencies(Target, "Flite");
		}
		PrivateDefinitions.Add("USING_FLITE=" + (bIsFliteUsed ? "1" : "0"));
	}
}
