{
	"FileVersion" : 3,
	"Version" : 1,
	"VersionName" : "0.1",
	"FriendlyName" : "TextToSpeech",
	"Description" : "A text to speech system that can be used to make auditory speech announcements given input strings.",
	"Category" : "Accessibility",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : false,
	"CanContainContent" : false,
	"IsExperimental" : true,
	"Installed" : false,
	"Modules" :
	[
		{
			"Name" : "TextToSpeech",
			"Type" : "Runtime",
			"LoadingPhase" : "PostEngineInit",
			"PlatformAllowList":
			[
				"Win64",
				"Mac",
				"Android",
				"IOS",
				"Linux",
				"LinuxArm64"
			],
			"TargetDenyList" :[
				"Server"
			]
		}
	]
}
